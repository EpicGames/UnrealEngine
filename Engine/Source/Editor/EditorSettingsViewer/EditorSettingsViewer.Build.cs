// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class EditorSettingsViewer : ModuleRules
	{
		public EditorSettingsViewer(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"CurveEditor",
					"EditorFramework",
					"Engine",
					"GraphEditor",
					"InputBindingEditor",
					"MessageLog",
					"SettingsEditor",
					"Slate",
					"SlateCore",
					"UnrealEd",
                    "InternationalizationSettings",
					"BlueprintGraph",
                    "Analytics",
                    "VREditor",
					"ToolMenus"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Settings",
				}
			);
		}
	}
}
