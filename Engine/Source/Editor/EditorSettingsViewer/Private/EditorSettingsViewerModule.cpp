// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analytics/AnalyticsPrivacySettings.h"
#include "Analytics/CrashReportsPrivacySettings.h"
#include "CrashReporterSettings.h"
#include "CurveEditorSettings.h"
#include "Delegates/Delegate.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Framework/Docking/TabManager.h"
#include "GraphEditorSettings.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModel.h"
#include "ISettingsEditorModule.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "ISettingsViewer.h"
#include "Internationalization/Internationalization.h"
#include "InternationalizationSettingsModel.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Preferences/PersonaOptions.h"
#include "Settings/ContentBrowserSettings.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Settings/EditorSettings.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Settings/SkeletalMeshEditorSettings.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Tests/AutomationTestSettings.h"
#include "Textures/SlateIcon.h"
#include "ToolMenuOwner.h"
#include "ToolMenus.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "VRModeSettings.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SNullWidget.h"

class SWidget;

#define LOCTEXT_NAMESPACE "FEditorSettingsViewerModule"

static const FName EditorSettingsTabName("EditorSettings");


/**
 * Implements the EditorSettingsViewer module.
 */
class FEditorSettingsViewerModule
	: public IModuleInterface
	, public ISettingsViewer
{
public:

	// ISettingsViewer interface

	virtual void ShowSettings( const FName& CategoryName, const FName& SectionName ) override
	{
		FGlobalTabmanager::Get()->TryInvokeTab(EditorSettingsTabName);
		ISettingsEditorModelPtr SettingsEditorModel = SettingsEditorModelPtr.Pin();

		if (SettingsEditorModel.IsValid())
		{
			ISettingsCategoryPtr Category = SettingsEditorModel->GetSettingsContainer()->GetCategory(CategoryName);

			if (Category.IsValid())
			{
				SettingsEditorModel->SelectSection(Category->GetSection(SectionName));
			}
		}
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			RegisterGeneralSettings(*SettingsModule);
			RegisterLevelEditorSettings(*SettingsModule);
			RegisterContentEditorsSettings(*SettingsModule);
			RegisterPrivacySettings(*SettingsModule);

			SettingsModule->RegisterViewer("Editor", *this);
		}

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(EditorSettingsTabName, FOnSpawnTab::CreateRaw(this, &FEditorSettingsViewerModule::HandleSpawnSettingsTab))
			.SetDisplayName(LOCTEXT("EditorSettingsTabTitle", "Editor Preferences"))
			.SetMenuType(ETabSpawnerMenuType::Hidden)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon"));

		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FEditorSettingsViewerModule::RegisterMenus));
	}

	virtual void ShutdownModule() override
	{
		UToolMenus::UnRegisterStartupCallback(this);
		UToolMenus::UnregisterOwner(this);

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(EditorSettingsTabName);
		UnregisterSettings();
	}

	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

protected:

	void RegisterMenus()
	{
		FToolMenuOwnerScoped OwnerScoped(this);

		GetMutableDefault<ULevelEditorPlaySettings>()->RegisterCommonResolutionsMenu();
	}

	/**
	 * Registers general Editor settings.
	 *
	 * @param SettingsModule A reference to the settings module.
	 */
	void RegisterGeneralSettings( ISettingsModule& SettingsModule )
	{
		// automation
		SettingsModule.RegisterSettings("Editor", "Advanced", "AutomationTest",
			LOCTEXT("AutomationSettingsName", "Automation"),
			LOCTEXT("AutomationSettingsDescription", "Set up automation test assets."),
			GetMutableDefault<UAutomationTestSettings>()
		);

		// region & language
		FModuleManager::Get().LoadModuleChecked("InternationalizationSettings");
		ISettingsSectionPtr RegionAndLanguageSettings = SettingsModule.RegisterSettings("Editor", "General", "Internationalization",
			LOCTEXT("InternationalizationSettingsModelName", "Region & Language"),
			LOCTEXT("InternationalizationSettingsModelDescription", "Configure the editor's behavior to use a language and fit a region's culture."),
			GetMutableDefault<UInternationalizationSettingsModel>()
		);

		// loading & saving features
		SettingsModule.RegisterSettings("Editor", "General", "LoadingSaving",
			LOCTEXT("LoadingSavingSettingsName", "Loading & Saving"),
			LOCTEXT("LoadingSavingSettingsDescription", "Change how the Editor loads and saves files."),
			GetMutableDefault<UEditorLoadingSavingSettings>()
		);

		// @todo thomass: proper settings support for source control module
		GetMutableDefault<UEditorLoadingSavingSettings>()->SccHackInitialize();

		// global editor settings
		SettingsModule.RegisterSettings("Editor", "General", "Global",
			LOCTEXT("GlobalSettingsName", "Global"),
			LOCTEXT("GlobalSettingsDescription", "Edit global settings that affect all editors."),
			GetMutableDefault<UEditorSettings>()
		);

		// misc unsorted settings
		SettingsModule.RegisterSettings("Editor", "General", "UserSettings",
			LOCTEXT("UserSettingsName", "Miscellaneous"),
			LOCTEXT("UserSettingsDescription", "Miscellaneous editor settings."),
			GetMutableDefault<UEditorPerProjectUserSettings>()
		);

		// Crash Reporter settings
		SettingsModule.RegisterSettings("Editor", "Advanced", "CrashReporter",
			LOCTEXT("CrashReporterSettingsName", "Crash Reporter"),
			LOCTEXT("CrashReporterSettingsDescription", "Various Crash Reporter related settings."),
			GetMutableDefault<UCrashReporterSettings>()
			);

		// experimental features
		SettingsModule.RegisterSettings("Editor", "General", "Experimental",
			LOCTEXT("ExperimentalettingsName", "Experimental"),
			LOCTEXT("ExperimentalSettingsDescription", "Enable and configure experimental Editor features."),
			GetMutableDefault<UEditorExperimentalSettings>()
		);

		// experimental features
		SettingsModule.RegisterSettings("Editor", "General", "VR Mode",
			LOCTEXT("VRModeSettings", "VR Mode"),
			LOCTEXT("VRModeSettingsDescription", "Configure VR editing features."),
			GetMutableDefault<UVRModeSettings>()
			);
	}

	/**
	 * Registers Level Editor settings.
	 *
	 * @param SettingsModule A reference to the settings module.
	 */
	void RegisterLevelEditorSettings( ISettingsModule& SettingsModule )
	{
		// play-in settings
		SettingsModule.RegisterSettings("Editor", "LevelEditor", "PlayIn",
			LOCTEXT("LevelEditorPlaySettingsName", "Play"),
			LOCTEXT("LevelEditorPlaySettingsDescription", "Set up window sizes and other options for the Play In Editor (PIE) feature."),
			GetMutableDefault<ULevelEditorPlaySettings>()
		);

		// viewport settings
		SettingsModule.RegisterSettings("Editor", "LevelEditor", "Viewport",
			LOCTEXT("LevelEditorViewportSettingsName", "Viewports"),
			LOCTEXT("LevelEditorViewportSettingsDescription", "Configure the look and feel of the Level Editor viewports."),
			GetMutableDefault<ULevelEditorViewportSettings>()
		);
	}

	/**
	 * Registers Other Tools settings.
	 *
	 * @param SettingsModule A reference to the settings module.
	 */
	void RegisterContentEditorsSettings( ISettingsModule& SettingsModule )
	{
		// content browser
		SettingsModule.RegisterSettings("Editor", "ContentEditors", "ContentBrowser",
			LOCTEXT("ContentEditorsContentBrowserSettingsName", "Content Browser"),
			LOCTEXT("ContentEditorsContentBrowserSettingsDescription", "Change the behavior of the Content Browser."),
			GetMutableDefault<UContentBrowserSettings>()
		);

		// skeletal mesh editor
		SettingsModule.RegisterSettings("Editor", "ContentEditors", "SkeletalMeshEditor",
			LOCTEXT("ContentEditorsSkeletalMeshEditorSettingsName", "Skeletal Mesh Editor"),
			LOCTEXT("ContentEditorsSkeletalMeshEditorSettingsDescription", "Change the behavior of the Skeletal Mesh Editor."),
			GetMutableDefault<USkeletalMeshEditorSettings>()
		);

		// graph editors
		SettingsModule.RegisterSettings("Editor", "ContentEditors", "GraphEditor",
			LOCTEXT("ContentEditorsGraphEditorSettingsName", "Graph Editors"),
			LOCTEXT("ContentEditorsGraphEditorSettingsDescription", "Customize Anim, Blueprint and Material Editor."),
			GetMutableDefault<UGraphEditorSettings>()
		);

		// Persona editors
		SettingsModule.RegisterSettings("Editor", "ContentEditors", "Persona",
			LOCTEXT("ContentEditorsPersonaSettingsName", "Animation Editor"),
			LOCTEXT("ContentEditorsPersonaSettingsDescription", "Customize Persona Editor."),
			GetMutableDefault<UPersonaOptions>()
		);

		// curve editor
		SettingsModule.RegisterSettings("Editor", "ContentEditors", "CurveEditor",
			LOCTEXT("ContentEditorsCurveEditorSettingsName", "Curve Editor"),
			LOCTEXT("ContentEditorsCurveEditorSettingsDescription", "Customize Curve Editors."),
			GetMutableDefault<UCurveEditorSettings>()
		);
	}

	void RegisterPrivacySettings(ISettingsModule& SettingsModule)
	{
		// Analytics
		SettingsModule.RegisterSettings("Editor", "Privacy", "Analytics",
			LOCTEXT("PrivacyAnalyticsSettingsName", "Usage Data"),
			LOCTEXT("PrivacyAnalyticsSettingsDescription", "Configure the way your Editor usage information is handled."),
			GetMutableDefault<UAnalyticsPrivacySettings>()
			);

		// Bug reports
		SettingsModule.RegisterSettings("Editor", "Privacy", "BugReports",
			LOCTEXT("PrivacyBugReportsSettingsName", "Bug Reports"),
			LOCTEXT("PrivacyBugReportsSettingsDescription", "Configure the way your bug reports information is handled."),
			GetMutableDefault<UCrashReportsPrivacySettings>()
		);
	}

	/** Unregisters all settings. */
	void UnregisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterViewer("Editor");

			// general settings
			SettingsModule->UnregisterSettings("Editor", "General", "InputBindings");
			SettingsModule->UnregisterSettings("Editor", "General", "LoadingSaving");
			SettingsModule->UnregisterSettings("Editor", "General", "GameAgnostic");
			SettingsModule->UnregisterSettings("Editor", "General", "UserSettings");
			SettingsModule->UnregisterSettings("Editor", "General", "AutomationTest");
			SettingsModule->UnregisterSettings("Editor", "General", "Internationalization");
			SettingsModule->UnregisterSettings("Editor", "General", "Experimental");
			SettingsModule->UnregisterSettings("Editor", "General", "CrashReporter");			

			// level editor settings
			SettingsModule->UnregisterSettings("Editor", "LevelEditor", "PlayIn");
			SettingsModule->UnregisterSettings("Editor", "LevelEditor", "Viewport");

			// other tools
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "ContentBrowser");
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "GraphEditor");
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "Persona");
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "CurveEditor");
		}
	}

private:

	/** Handles creating the editor settings tab. */
	TSharedRef<SDockTab> HandleSpawnSettingsTab( const FSpawnTabArgs& SpawnTabArgs )
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		TSharedRef<SWidget> SettingsEditor = SNullWidget::NullWidget;

		if (SettingsModule != nullptr)
		{
			ISettingsContainerPtr SettingsContainer = SettingsModule->GetContainer("Editor");

			if (SettingsContainer.IsValid())
			{
				ISettingsEditorModule& SettingsEditorModule = FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor");
				ISettingsEditorModelRef SettingsEditorModel = SettingsEditorModule.CreateModel(SettingsContainer.ToSharedRef());

				SettingsEditor = SettingsEditorModule.CreateEditor(SettingsEditorModel);
				SettingsEditorModelPtr = SettingsEditorModel;
			}
		}

		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SettingsEditor
			];
	}

private:

	/** Holds a pointer to the settings editor's view model. */
	TWeakPtr<ISettingsEditorModel> SettingsEditorModelPtr;
};


IMPLEMENT_MODULE(FEditorSettingsViewerModule, EditorSettingsViewer);


#undef LOCTEXT_NAMESPACE
